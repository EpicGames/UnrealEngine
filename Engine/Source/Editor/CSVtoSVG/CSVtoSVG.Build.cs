// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class CSVtoSVG : ModuleRules
{
	public CSVtoSVG(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"EditorConfig",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"WorkspaceMenuStructure",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"ContentBrowserData",
				"Settings",
            }
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
                "AssetTools",
				"EditorWidgets",
			}
		);
	}
}
