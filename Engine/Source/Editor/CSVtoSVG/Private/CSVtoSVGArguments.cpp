// Copyright Epic Games, Inc. All Rights Reserved.

#include "CSVtoSVGArguments.h"

#include "Misc/Paths.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CSVtoSVGArguments)

UCSVtoSVGArugments::UCSVtoSVGArugments(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

FString UCSVtoSVGArugments::GetCommandLine() const
{
	if (CSV.FilePath.IsEmpty())
	{
		return FString();
	}

	if (OutputDirectory.Path.IsEmpty())
	{
		return FString();
	}

	if (OutputFilename.IsEmpty())
	{
		return FString();
	}

	// Get the default argument values.
	UCSVtoSVGArugments* Default = CastChecked<UCSVtoSVGArugments>(GetClass()->GetDefaultObject());

	// Programmatically generate a list of optional arugments by iterating reflected properties.
	// We use the property name as the argument name to CSVtoSVG.
	FString Arguments;
	for (TFieldIterator<FProperty> It(GetClass()); It; ++It)
	{
		const FProperty* Property = *It;
		if (Property && Property->HasMetaData("Category") && (Property->GetMetaData("Category") == TEXT("OptionalArguments")))
		{
			const FString ArgumentName = Property->GetName();

			FString ArgumentValue;
			const bool bDifferentThanDefault = Property->ExportText_InContainer(0, ArgumentValue, this, Default, nullptr, PPF_None);

			// If the user has set a value different from the defaults use the argument.
			if (bDifferentThanDefault)
			{
				// Bool properties are an exception.  They are just commandline switches.
				if (Property->GetCPPType() == TEXT("bool"))
				{
					Arguments += FString::Printf(TEXT(" -%s"), *ArgumentName);
				}
				else
				{
					Arguments += FString::Printf(TEXT(" -%s %s"), *ArgumentName, *ArgumentValue);
				}
			}
		}
	}

	// Build an argument string with the required and optional arugments.
	return FString::Printf(
		TEXT("-csv %s -o %s %s"),
		*CSV.FilePath,
		*GetOutputFileName(),
		*Arguments
	);
}

FString UCSVtoSVGArugments::GetOutputFileName() const
{
	return FPaths::Combine(*OutputDirectory.Path, OutputFilename);
}
