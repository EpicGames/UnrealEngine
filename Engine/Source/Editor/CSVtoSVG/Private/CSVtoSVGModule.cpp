// Copyright Epic Games, Inc. All Rights Reserved.

#include "CSVtoSVGModule.h"

#include "CSVtoSVGArguments.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogCategory.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "SCSVtoSVG.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "CSVtoSVG"

IMPLEMENT_MODULE(FCSVtoSVGModule, CSVtoSVG);
DEFINE_LOG_CATEGORY(LogCSVtoSVG);

namespace FCSVtoSVG
{
	static const FName CSVtoSVGApp = FName(TEXT("CSVtoSVGApp"));
}

TSharedRef<SDockTab> FCSVtoSVGModule::CreateTab(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> MajorTab;
	SAssignNew(MajorTab, SDockTab)
		.TabRole(ETabRole::MajorTab);

	MajorTab->SetContent(SNew(SCSVtoSVG));

	return MajorTab.ToSharedRef();
}


void FCSVtoSVGModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FCSVtoSVG::CSVtoSVGApp, FOnSpawnTab::CreateRaw(this, &FCSVtoSVGModule::CreateTab))
		.SetDisplayName(NSLOCTEXT("CSVtoSVGApp", "TabTitle", "CSV to SVG"))
		.SetTooltipText(NSLOCTEXT("CSVtoSVGApp", "TooltipText", "Tool for generating vector line graphs from comma-separated value files generated from CSV profiles."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.UserDefinedStruct"));
}

void FCSVtoSVGModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FCSVtoSVG::CSVtoSVGApp);
	}
}

FFilePath GenerateSVG(const UCSVtoSVGArugments& Arguments, const TArray<FString>& StatList)
{
	const FString CSVtoSVGExe = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/DotNET/CsvTools"), TEXT("CSVToSVG.exe"));

	FString CmdLine = Arguments.GetCommandLine();
	if (CmdLine.IsEmpty())
	{
		return FFilePath();
	}

	// Add selected stats to the command line.
	CmdLine += FString(TEXT(" -stats"));
	for (const FString& Stat : StatList)
	{
		CmdLine += TEXT(" ") + Stat;
	}

	UE_LOGF(LogCSVtoSVG, Display, "Generating SVG: %ls %ls", *CSVtoSVGExe, *CmdLine);

	// Create a read and write pipe for the child process
	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;
	if (!FPlatformProcess::CreatePipe(PipeRead, PipeWrite))
	{
		return FFilePath();
	}

	FProcHandle ProcHandle = FPlatformProcess::CreateProc(*CSVtoSVGExe, *CmdLine, true, false, false, nullptr, 0, nullptr, PipeWrite);
	if (ProcHandle.IsValid())
	{
		FString CSVtoCSVOutput;
		while (FPlatformProcess::IsProcRunning(ProcHandle))
		{
			CSVtoCSVOutput += FPlatformProcess::ReadPipe(PipeRead);

		}
		FPlatformProcess::WaitForProc(ProcHandle);

		if (!CSVtoCSVOutput.IsEmpty())
		{
			UE_LOGF(LogCSVtoSVG, Display, "%ls", *CSVtoCSVOutput);
		}

		if (!CSVtoCSVOutput.Contains(TEXT("[ERROR]")))
		{
			// If there was no error open the output file.
			const FString OutputFileName = Arguments.GetOutputFileName();
			if (!OutputFileName.IsEmpty())
			{
				FPlatformProcess::LaunchURL(*OutputFileName, nullptr, nullptr);
			}
		}

		FPlatformProcess::CloseProc(ProcHandle);
	}

	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);

	return FFilePath();
}

#undef LOCTEXT_NAMESPACE
