// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCSVtoSVG.h"

#include "CSVtoSVGArguments.h"
#include "CSVtoSVGModule.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SStatList.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Less.h"
#include "Types/SlateEnums.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "CSVtoSVG"

void SCSVtoSVG::Construct(const FArguments& Args)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.bShowOptions = true;
	DetailsViewArgs.bShowModifiedPropertiesOption = true;

	Arguments = TStrongObjectPtr<UCSVtoSVGArugments>(NewObject<UCSVtoSVGArugments>());
	Arguments->LoadEditorConfig();

	CSVtoSVGArgumentsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	CSVtoSVGArgumentsDetailsView->SetObject(Arguments.Get());

	const FMargin PaddingAmount(5.0f);

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(PaddingAmount)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(PaddingAmount)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				.Padding(PaddingAmount)
				[
					CSVtoSVGArgumentsDetailsView->AsShared()
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(PaddingAmount)
			[
				SAssignNew(StatListView, SStatList)
			]
		]
		+ SVerticalBox::Slot()
			.Padding(PaddingAmount)
			.AutoHeight()
			.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ForegroundColor(FLinearColor::White)
			.OnClicked(FOnClicked::CreateSP(this, &SCSVtoSVG::OnGenerateGraphClicked))
			.ToolTipText(LOCTEXT("GenerateSVGGraphTooltipLoc", "Generates an SVG graph."))
			.Text(LOCTEXT("GenerateSVGGraphLoc", "Generate"))
		]
	];

	// Load CSV, which in turn will populate the UI.
	if (!Arguments->CSV.FilePath.IsEmpty())
	{
		LoadCSVFile(*Arguments->CSV.FilePath);
	}
}

/** SCSVtoSVG destructor */
SCSVtoSVG::~SCSVtoSVG()
{
	Arguments->SaveEditorConfig();
}

FReply SCSVtoSVG::OnGenerateGraphClicked()
{
	FFilePath OutputFilePath = GenerateSVG(*Arguments, StatListView->GetSelectedStats());
	if (!OutputFilePath.FilePath.IsEmpty())
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SCSVtoSVG::StatListSelectionChanged(const TArray<FString>& Stats)
{
	StatListView->UpdateStatList(Stats);
}

void SCSVtoSVG::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetName() == TEXT("CSV"))
	{
		LoadCSVFile(*Arguments->CSV.FilePath);
	}
}

void SCSVtoSVG::LoadCSVFile(const FString& Filepath)
{
	TArray<FString> Contents;
	if (FFileHelper::LoadFileToStringArray(Contents, *Filepath))
	{
		if (Contents.Num())
		{
			const FString& StatLine = Contents[0];
			TArray<FString> Stats;
			StatLine.ParseIntoArray(Stats, TEXT(","));

			Stats.Sort(TLess<FString>());

			StatListSelectionChanged(Stats);
		}

		// If the user has not specified a value for filename infer one from the source csv file.
		if (Arguments->OutputFilename.IsEmpty())
		{
			Arguments->OutputFilename = FPaths::ChangeExtension(FPaths::GetCleanFilename(Arguments->CSV.FilePath), TEXT("svg"));
		}

		// If the user has not specified a value for directory infer ones from the source csv file directory
		if (Arguments->OutputDirectory.Path.IsEmpty())
		{
			Arguments->OutputDirectory.Path = FPaths::ConvertRelativePathToFull(FPaths::GetPath(Arguments->CSV.FilePath));
		}
	}
}

#undef LOCTEXT_NAMESPACE
