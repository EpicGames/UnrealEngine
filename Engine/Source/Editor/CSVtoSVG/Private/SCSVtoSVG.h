// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CSVtoSVGArguments.h"
#include "Input/Reply.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FProperty;
class IDetailsView;
class SStatList;
class UCSVtoSVGArugments;
struct FPropertyChangedEvent;

class SCSVtoSVG : public SCompoundWidget, public FNotifyHook
{
	SLATE_BEGIN_ARGS(SCSVtoSVG)
	{
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	/** SCSVtoSVG destructor */
	~SCSVtoSVG();

private:

	FReply OnGenerateGraphClicked();

	void StatListSelectionChanged(const TArray<FString>& Stats);

	//~ Begin FNotifyHook Interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	// End of FNotifyHook

	void LoadCSVFile(const FString& Filepath);

	/** This is the Detail View for the currently selected UActorRecording or an item from an Extender */
	TSharedPtr<IDetailsView> CSVtoSVGArgumentsDetailsView = nullptr;

	/** The list of stats parsed from the CSV file. */
	TSharedPtr<SStatList> StatListView = nullptr;

	/** The current arguments. */
	TStrongObjectPtr<UCSVtoSVGArugments> Arguments = nullptr;
};
