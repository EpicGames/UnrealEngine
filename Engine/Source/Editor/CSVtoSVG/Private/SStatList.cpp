// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStatList.h"

#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "CSVtoSVG"

void SStatList::Construct(const FArguments& Args)
{
	StatListView = SNew(SListView<TSharedPtr<FString>>)
					.ListItemsSource(&StatList)
					.OnGenerateRow(this, &SStatList::OnGenerateWidgetForList);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			StatListView.ToSharedRef()
		]
	];
}

TSharedRef<ITableRow> SStatList::OnGenerateWidgetForList(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
		SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock)
				.Text(FText::FromString(*InItem.Get()))
		];
}

void SStatList::UpdateStatList(const TArray<FString>& StatNames)
{
	StatList.Empty();

	for (const auto& StatName : StatNames)
	{
		StatList.Add(MakeShared<FString>(StatName));
	}

	StatListView->RequestListRefresh();
}

TArray<FString> SStatList::GetSelectedStats() const
{
	TArray<TSharedPtr<FString>> SelectedItems = StatListView->GetSelectedItems();

	TArray<FString> SelectedStats;
	for (const TSharedPtr<FString>& Item : SelectedItems)
	{
		if (Item.IsValid())
		{
			SelectedStats.Add(*Item);
		}
	}

	return SelectedStats;
}

#undef LOCTEXT_NAMESPACE
