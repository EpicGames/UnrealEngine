// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;

class SStatList : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SStatList)
	{
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	void UpdateStatList(const TArray<FString>& StatNames);

	TArray<FString> GetSelectedStats() const;

private:

	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<SListView<TSharedPtr<FString>>> StatListView;
	TArray<TSharedPtr<FString>> StatList;
};
