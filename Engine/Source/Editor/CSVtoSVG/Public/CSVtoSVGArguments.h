// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EditorConfigBase.h"
#include "Engine/EngineTypes.h"
#include "HAL/PlatformMath.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

#include "CSVtoSVGArguments.generated.h"

class FProperty;
class UObject;

DECLARE_MULTICAST_DELEGATE_TwoParams(FCSVtoSVGArugmentsPropertySetModifiedSignature, UObject*, FProperty*);

UENUM()
enum class ESVGTheme
{
	Dark,
	Light,
};

UCLASS(EditorConfig = "CSVtoSVG")
class UCSVtoSVGArugments : public UEditorConfigBase
{
	GENERATED_UCLASS_BODY()

	/** The base directories to be considered Internal Only for the struct picker.*/
	UPROPERTY(EditAnywhere, Category = RequiredArguments, meta = (EditorConfig, DisplayName = "CSV", Tooltip = "Specifies a list of CSVs separated by spaces"))
	FFilePath CSV;

	UPROPERTY(EditAnywhere, Category = RequiredArguments, meta = (EditorConfig, DisplayName = "Ouput Directory", Tooltip = "Sets the filename for the SVG output."))
	FDirectoryPath OutputDirectory;

	UPROPERTY(EditAnywhere, Category = RequiredArguments, meta = (EditorConfig, DisplayName = "Ouput Filename", Tooltip = "Sets the filename for the SVG output."))
	FString OutputFilename;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Skip Rows", Tooltip = "Skips a specified number of rows in the CSV. This is useful for CSV file sgenerated from the FPSChartStart command, where there's a 4-row summary at the top."))
	int32 skipRows = 0;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "minX", Tooltip = "Clamps the X and Y range of the source data."))
	float minX = -100000.0f;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "maxX ", Tooltip = "Clamps the X and Y range of the source data."))
	float maxX = -100000.0f;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "minY", Tooltip = "Clamps the X and Y range of the source data."))
	float minY = -100000.0f;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "maxY", Tooltip = "Clamps the X and Y range of the source data."))
	float maxY = -100000.0f;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Smooth", Tooltip = "Smooth the graph."))
	bool smooth = false;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Smooth kernel size", Tooltip = "Specifies the smoothing kernel size in column entries to use."))
	int32 smoothKernelSize = -1;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Smooth kernel size", Tooltip = "Specifies the smoothing kernel size as a percentage of the graph length."))
	float smoothKernelPercent = -1.0f;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Width", Tooltip = "The width of the graph."))
	int32 width = 1000;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Height", Tooltip = "The height of the graph."))
	int32 height = 500;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Title", Tooltip = "Sets the title for the graph."))
	FString title;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "No Metadata", Tooltip = ""))
	bool noMetadata = false;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Graph only", Tooltip = "Render just the graph, no borders."))
	bool graphOnly = false;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Budget [ms]", Tooltip = "Sets the budget line. Default is 33.3."))
	float budget = 33.3f;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Thickness", Tooltip = "Sets the line thickness of the graph."))
	float thickness = 1.0f;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Theme", Tooltip = ""))
	ESVGTheme theme = ESVGTheme::Dark;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Threshold", Tooltip = "Ignores stats which are entirely under this threshold."))
	float threshold = -FLT_MAX;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Stacked", Tooltip = "Makes a stacked graph for cumulative stats."))
	bool stacked = false;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Stacked Total Stat", Tooltip = "Specifies the total stat. Valid for stacked graphs only."))
	FString stacktotalstack;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Interactive", Tooltip = "Adds an interactive frame marker."))
	bool interactive = false;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Show Averages", Tooltip = "Shows stat averages next to the legend and sorts the stats in the legend high to low by average value."))
	bool showaverages = false;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Colour Offset", Tooltip = "Rotates the colours."))
	int colourOffset = 0;

	UPROPERTY(EditAnywhere, Category = OptionalArguments, meta = (EditorConfig, DisplayName = "Average Threshold", Tooltip = "Ignores stats whose average is under this threshold."))
	float averageThreshold = -FLT_MAX;

	FString GetCommandLine() const;

	FString GetOutputFileName() const;

	/** @return the multicast delegate that is called when properties are modified */
	FCSVtoSVGArugmentsPropertySetModifiedSignature& GetOnModified()
	{
		return OnModified;
	}

private:
	FCSVtoSVGArugmentsPropertySetModifiedSignature OnModified;
};
