// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPath.h"

class FSpawnTabArgs;
class FString;
class SDockTab;
class UCSVtoSVGArugments;

DECLARE_LOG_CATEGORY_EXTERN(LogCSVtoSVG, Log, All);

/**
 * Generates a SVG graph from the given arguments.
 *
 * @param	Arguments								The arguments to define and configure graph generation.
 *
 * @return	File path to the generated SVG file.
 */
FFilePath GenerateSVG(const UCSVtoSVGArugments& Arguments, const TArray<FString>& StatList);

/**
 * Struct Viewer module
 */
class FCSVtoSVGModule : public IModuleInterface
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

private:
	TSharedRef<SDockTab> CreateTab(const FSpawnTabArgs& Args);
};
